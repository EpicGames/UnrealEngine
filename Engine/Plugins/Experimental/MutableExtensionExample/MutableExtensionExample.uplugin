{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Mutable Extension Example",
	"Description": "Example plugin which adds new external operations to Mutable.",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"EnabledByDefault": false,
	"Modules": 
	[
		{
			"Name": "MutableExtensionExample",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": 
	[
		{
			"Name": "Mutable",
			"Enabled": true
		}
	]
}