// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MutableExtensionExample : ModuleRules
	{
        public MutableExtensionExample(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateIncludePaths.Add(ModuleDirectory + "/Private");
			PublicIncludePaths.Add(ModuleDirectory + "/Public");
			
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"MutableRuntime"
				});

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"CustomizableObject"
				});
		}
	}
}
