// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshClipSphere.h"

#include "PrimitiveSphere.h"
#include "MuR/External/MeshAdapter.h"

#define LOCTEXT_NAMESPACE "MutableExtensionExample"


const FText FMeshClipSphere::TextInputMesh = LOCTEXT("Mesh", "Mesh");

const FText FMeshClipSphere::TextInputPrimitiveSphere = LOCTEXT("Sphere", "Sphere");


TArray<TPair<FText, const UScriptStruct*>> FMeshClipSphere::GetInputs() const
{
	TArray<TPair<FText, const UScriptStruct*>> Inputs;

	Inputs.Emplace(TextInputMesh, UE::Mutable::FMeshAdapter::StaticStruct());
	Inputs.Emplace(TextInputPrimitiveSphere, FPrimitiveSphere::StaticStruct());
	
	return Inputs;
}


TPair<FText, const UScriptStruct*> FMeshClipSphere::GetOutput() const
{
	return MakeTuple(LOCTEXT("Mesh", "Mesh"), UE::Mutable::FMeshAdapter::StaticStruct());
}


void FMeshClipSphere::Evaluate(UE::Mutable::FContext& Context) const
{
	// Since we are going to edit the Mesh input, we need to make it non-constant using the CopyOrMove.
	UE::Mutable::FValue InputMesh = UE::Mutable::CopyOrMove(Context.GetInput(TextInputMesh));

	// Cast the input to a Mesh Adapter.
	UE::Mutable::FMeshAdapter& Mesh = InputMesh.Get<UE::Mutable::FMeshAdapter>();

	// Get the Primitive input
	UE::Mutable::FValueConst InputSphere = Context.GetInput(TextInputPrimitiveSphere);
	const FPrimitiveSphere& PrimitiveSphere = InputSphere.Get<FPrimitiveSphere>();
	
	TBitArray<> VerticesToClip;
	VerticesToClip.Init(false, Mesh.GetNumVertices());

	for (int32 Index = 0; Index < Mesh.GetNumVertices(); ++Index)
	{
		FVector3f Vertex = Mesh.GetVertex(Index);

		if (PrimitiveSphere.Sphere.IsInside(static_cast<FVector>(Vertex)))
		{
			VerticesToClip[Index] = true;
		}
	}

	Mesh.RemoveVertices(VerticesToClip);
	
	// Output the modified Mesh Adapter.
	Context.SetOutput(MoveTemp(InputMesh));
}


#undef LOCTEXT_NAMESPACE