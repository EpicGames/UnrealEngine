// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshIdentity.h"

#include "MuR/External/MeshAdapter.h"

#define LOCTEXT_NAMESPACE "MutableExtensionExample"


const FText FMeshIdentity::TextInputMesh = LOCTEXT("Mesh", "Mesh");


TArray<TPair<FText, const UScriptStruct*>> FMeshIdentity::GetInputs() const
{
	TArray<TPair<FText, const UScriptStruct*>> Inputs;

	Inputs.Emplace(TextInputMesh, UE::Mutable::FMeshAdapter::StaticStruct());
	
	return Inputs;
}


TPair<FText, const UScriptStruct*> FMeshIdentity::GetOutput() const
{
	return MakeTuple(LOCTEXT("Mesh", "Mesh"), UE::Mutable::FMeshAdapter::StaticStruct());
}


void FMeshIdentity::Evaluate(UE::Mutable::FContext& Context) const
{
	Context.SetOutput(Context.GetInput(TextInputMesh));
}


#undef LOCTEXT_NAMESPACE