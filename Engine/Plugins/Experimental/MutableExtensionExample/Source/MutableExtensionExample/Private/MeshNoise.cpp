// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshNoise.h"

#include "MuR/External/FloatAdapter.h"
#include "MuR/External/MeshAdapter.h"

#define LOCTEXT_NAMESPACE "MutableExtensionExample"


const FText FMeshNoise::TextInputMesh = LOCTEXT("Mesh", "Mesh");

const FText FMeshNoise::TextInputFactor = LOCTEXT("Factor", "Factor");


TArray<TPair<FText, const UScriptStruct*>> FMeshNoise::GetInputs() const
{
	TArray<TPair<FText, const UScriptStruct*>> Inputs;

	Inputs.Emplace(TextInputMesh, UE::Mutable::FMeshAdapter::StaticStruct());
	Inputs.Emplace(TextInputFactor, UE::Mutable::FFloatAdapter::StaticStruct());
	
	return Inputs;
}


TPair<FText, const UScriptStruct*> FMeshNoise::GetOutput() const
{
	return MakeTuple(LOCTEXT("Mesh", "Mesh"), UE::Mutable::FMeshAdapter::StaticStruct());
}


void FMeshNoise::Evaluate(UE::Mutable::FContext& Context) const
{
	// Since we are going to edit the Mesh input, we need to make it non-constant using the CopyOrMove.
	UE::Mutable::FValue Input = UE::Mutable::CopyOrMove(Context.GetInput(TextInputMesh));

	// Cast the input to a Mesh Adapter.
	UE::Mutable::FMeshAdapter& Mesh = Input.Get<UE::Mutable::FMeshAdapter>();

	// Get the Factor input
	const float Factor = Context.GetInput(TextInputFactor).Get<UE::Mutable::FFloatAdapter>().GetValue();
	
	// Move each vertex some random noise. This random is allowed since it is deterministic.
	FRandomStream Random(Seed);
	
	for (int32 Index = 0; Index < Mesh.GetNumVertices(); ++Index)
	{
		FVector3f Vertex = Mesh.GetVertex(Index);
		
		Vertex += static_cast<FVector3f>(Random.GetUnitVector()) * Factor;
		
		Mesh.SetVertex(Index, Vertex);
	}
	
	// Output the modified Mesh Adapter.
	Context.SetOutput(MoveTemp(Input));
}


#undef LOCTEXT_NAMESPACE