// Copyright Epic Games, Inc. All Rights Reserved.

#include "MutableExtensionExampleModule.h"

#include "MeshClipSphere.h"
#include "MeshIdentity.h"
#include "MeshNoise.h"
#include "MuCO/ICustomizableObjectModule.h"


IMPLEMENT_MODULE(FMutableExtensionExampleModule, MutableExtensionExample);


void FMutableExtensionExampleModule::StartupModule()
{
	ICustomizableObjectModule& Module = ICustomizableObjectModule::Get();

	Module.RegisterExternalOperation(FMeshIdentity::StaticStruct());
	Module.RegisterExternalOperation(FMeshNoise::StaticStruct());
	Module.RegisterExternalOperation(FMeshClipSphere::StaticStruct());
}


void FMutableExtensionExampleModule::ShutdownModule()
{
	ICustomizableObjectModule& Module = ICustomizableObjectModule::Get();

	Module.UnregisterExternalOperation(FMeshIdentity::StaticStruct());
	Module.UnregisterExternalOperation(FMeshNoise::StaticStruct());
	Module.UnregisterExternalOperation(FMeshClipSphere::StaticStruct());
}
