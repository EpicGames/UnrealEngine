// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/External/Operation.h"

#include "MeshClipSphere.generated.h"


USTRUCT(DisplayName = "Mesh Clip With Sphere")
struct FMeshClipSphere : public UE::Mutable::FExternalOperation
{
	GENERATED_BODY()
	
	virtual ~FMeshClipSphere() override = default;
	virtual TArray<TPair<FText, const UScriptStruct*>> GetInputs() const override;
	virtual TPair<FText, const UScriptStruct*> GetOutput() const override;
	virtual void Evaluate(UE::Mutable::FContext& Context) const override;

	const static FText TextInputMesh;

	const static FText TextInputPrimitiveSphere;
};