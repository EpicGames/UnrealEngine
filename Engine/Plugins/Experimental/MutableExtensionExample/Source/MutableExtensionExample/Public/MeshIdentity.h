// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/External/Operation.h"

#include "MeshIdentity.generated.h"


USTRUCT(DisplayName = "Mesh Identity")
struct FMeshIdentity : public UE::Mutable::FExternalOperation
{
	GENERATED_BODY()
	
	virtual ~FMeshIdentity() override = default;
	virtual TArray<TPair<FText, const UScriptStruct*>> GetInputs() const override;
	virtual TPair<FText, const UScriptStruct*> GetOutput() const override;
	virtual void Evaluate(UE::Mutable::FContext& Context) const override;

	const static FText TextInputMesh;
};