// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/External/Operation.h"

#include "MeshNoise.generated.h"


USTRUCT(DisplayName = "Mesh Noise")
struct FMeshNoise : public UE::Mutable::FExternalOperation
{
	GENERATED_BODY()
	
	virtual TArray<TPair<FText, const UScriptStruct*>> GetInputs() const override;
	virtual TPair<FText, const UScriptStruct*> GetOutput() const override;
	virtual void Evaluate(UE::Mutable::FContext& Context) const override;

	const static FText TextInputMesh;

	const static FText TextInputFactor;

	/** Operation constant. This value will be copied when compiling a Customizable Object, hence it can not be changed at runtime.
	 *
	 * If a value needs to change at runtime, it must be a Parameter. */
	UPROPERTY(EditAnywhere, Category = "Mesh Noise")
	int32 Seed = 0;
};