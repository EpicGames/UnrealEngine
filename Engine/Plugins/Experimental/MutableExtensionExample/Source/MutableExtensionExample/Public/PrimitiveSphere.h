// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PrimitiveSphere.generated.h"


USTRUCT(DisplayName = "Sphere")
struct FPrimitiveSphere
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Primitive)
	FSphere3d Sphere;
};